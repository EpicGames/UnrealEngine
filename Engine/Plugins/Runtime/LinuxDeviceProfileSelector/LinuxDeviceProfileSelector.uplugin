{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Linux Device Profile Selector",
	"Description" : "Linux Device Profile Selector",
	"Category" : "Device Profile Selectors",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "LinuxDeviceProfileSelector",
			"Type" : "RuntimeNoCommandlet",
			"LoadingPhase" : "PostConfigInit",
			"PlatformAllowList" :
			[
				"Linux",
				"LinuxArm64"
			]
		}
	]
}
