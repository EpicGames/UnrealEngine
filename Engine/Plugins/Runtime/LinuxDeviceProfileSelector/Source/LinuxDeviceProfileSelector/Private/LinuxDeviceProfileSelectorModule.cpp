// Copyright Epic Games, Inc. All Rights Reserved.

#include "LinuxDeviceProfileSelectorModule.h"
#include "Modules/ModuleManager.h"
#include "IHeadMountedDisplayModule.h"
#include "StereoRendering.h"

IMPLEMENT_MODULE(FLinuxDeviceProfileSelectorModule, LinuxDeviceProfileSelector);


void FLinuxDeviceProfileSelectorModule::StartupModule()
{
}


void FLinuxDeviceProfileSelectorModule::ShutdownModule()
{
}


FString const FLinuxDeviceProfileSelectorModule::GetRuntimeDeviceProfileName()
{
	static FString ProfileName;

	if (ProfileName.IsEmpty())
	{
#if UE_IS_COOKED_EDITOR
		// some heuristics to determine a cooked editor
		ProfileName = TEXT("LinuxCookedEditor");
#else
		// [RCL] 2015-09-22 FIXME: support different environments
		ProfileName = FPlatformProperties::PlatformName();

		if (!IsRunningDedicatedServer() && !GIsEditor && IHeadMountedDisplayModule::IsAvailable())
		{
			IHeadMountedDisplayModule& HMDModule = IHeadMountedDisplayModule::Get();
			if (HMDModule.IsStandaloneStereoOnlyDevice())
			{
				// Add support for proper profile identification on Linux platforms that support XR
				FString HMDRequestedProfileName = HMDModule.GetDeviceSystemName();

				if (!HMDRequestedProfileName.IsEmpty())
				{
					FString RuntimeName, DeviceDriverName;

					// if DeviceSystemName includes a "Runtime:Device" use the Device part.  Otherwise just use the whole string
					if (HMDRequestedProfileName.Split(TEXT(":"), &RuntimeName, &DeviceDriverName))
					{
						DeviceDriverName.TrimStartAndEndInline();

						if (!DeviceDriverName.IsEmpty())
						{
							ProfileName = DeviceDriverName;
						}
					}
					else
					{
						ProfileName = HMDRequestedProfileName;
					}
				}
			}
		}
#endif
	}

	UE_LOGF(LogLinux, Log, "Selected Device Profile: [%ls]", *ProfileName);
	return ProfileName;
}

