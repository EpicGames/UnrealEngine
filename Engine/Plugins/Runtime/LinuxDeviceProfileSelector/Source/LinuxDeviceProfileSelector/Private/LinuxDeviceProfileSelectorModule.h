// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDeviceProfileSelectorModule.h"

/**
 * Implements the Linux Device Profile Selector module.
 */
class FLinuxDeviceProfileSelectorModule
	: public IDeviceProfileSelectorModule
{
public:

	// Begin IDeviceProfileSelectorModule interface
	virtual const FString GetRuntimeDeviceProfileName() override;
	// End IDeviceProfileSelectorModule interface


	// Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End IModuleInterface interface

	
	/**
	 * Virtual destructor.
	 */
	virtual ~FLinuxDeviceProfileSelectorModule()
	{
	}
};
